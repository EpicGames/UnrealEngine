// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericPlatformInstallation.h"
#include "../PlatformInstallation.h"
#include "DesktopPlatformModule.h"
#include "Misc/Paths.h"

bool FGenericPlatformInstallation::NormalizeEngineRootDir(FString &RootDir)
{
	// Canonicalize the engine path and remove the last backslash.
	FString NormalizedRootDir = RootDir;
	FPaths::NormalizeDirectoryName(NormalizedRootDir);

	// Check if it's valid
	if (FDesktopPlatformModule::Get()->IsValidRootDirectory(NormalizedRootDir))
	{
		RootDir = NormalizedRootDir;
		return true;
	}

	// Otherwise try to accept directories underneath the root
	if (!NormalizedRootDir.RemoveFromEnd(TEXT("/Engine")))
	{
		if (!NormalizedRootDir.RemoveFromEnd(TEXT("/Engine/Binaries")))
		{
			NormalizedRootDir.RemoveFromEnd(FString(TEXT("/Engine/Binaries/")) / FPlatformProcess::GetBinariesSubdirectory());
		}
	}

	// Check if the engine binaries directory exists
	if (FDesktopPlatformModule::Get()->IsValidRootDirectory(NormalizedRootDir))
	{
		RootDir = NormalizedRootDir;
		return true;
	}

	return false;
}

bool FGenericPlatformInstallation::LaunchEditor(const FString& RootDirName, const FString &FileName, const FString &Arguments)
{
	// No default implementation
	return false;
}

bool FGenericPlatformInstallation::SelectEngineInstallation(FString &Identifier)
{
	// No default implementation
	return false;
}

void FGenericPlatformInstallation::ErrorDialog(const FString& Message, const FString& LogText)
{
	// No default implementation
}