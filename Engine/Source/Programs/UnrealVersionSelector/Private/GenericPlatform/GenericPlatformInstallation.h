// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FGenericPlatformInstallation
{
	// Validate and normalize an engine root directory name
	static bool NormalizeEngineRootDir(FString &RootDir);

	// Launches the editor application
	static bool LaunchEditor(const FString& RootDirName, const FString &ExplicitFileName, const FString &Arguments);

	// Select an engine installation
	static bool SelectEngineInstallation(FString &Identifier);

	// Shows an error dialog with log output
	static void ErrorDialog(const FString &Message, const FString &LogText);
};
