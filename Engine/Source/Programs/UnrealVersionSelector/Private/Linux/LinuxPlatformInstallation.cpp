// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinuxPlatformInstallation.h"

#include "DesktopPlatformModule.h"
#include "Misc/Paths.h"
#include "Framework/Application/SlateApplication.h"
#include "SlateBasics.h"
#include "StandaloneRenderer.h"
#include "SlateFileDialogsStyles.h"

#define LOCTEXT_NAMESPACE "UnrealVersionSelector"

struct FEngineLabelSortPredicate
{
	bool operator()(const FString &A, const FString &B) const
	{
		return FDesktopPlatformModule::Get()->IsPreferredEngineIdentifier(A, B);
	}
};

FString GetInstallationDescription(const FString &Id, const FString &RootDir)
{
	// Official release versions just have a version number
	if (Id.Len() > 0 && FChar::IsDigit(Id[0]))
	{
		return Id;
	}

	// Otherwise get the path
	FString PlatformRootDir = RootDir;
	FPaths::MakePlatformFilename(PlatformRootDir);

	// Perforce build
	if (FDesktopPlatformModule::Get()->IsSourceDistribution(RootDir))
	{
		return FString::Printf(TEXT("Source build at %s"), *PlatformRootDir);
	}
	else
	{
		return FString::Printf(TEXT("Binary build at %s"), *PlatformRootDir);
	}
}

static void InitSlate()
{
#if WITH_EDITOR
	FCoreStyle::ResetToDefault();
	
	FModuleManager::Get().LoadModuleChecked("EditorStyle");
#endif
	FSlateApplication::InitializeAsStandaloneApplication(GetStandardStandaloneRenderer());
}

static void CleanupSlate()
{
	FSlateApplication::Shutdown();
}

struct FEngineInstallationInfo
{
	FString Identifier;
	FString Description;
};

struct FSelectBuildInfo
{
	FString Identifier;
	TSharedPtr<FEngineInstallationInfo> SelectedEngineInstallationInfo;
	TMap<FString, FString> Installations;
	TArray<TSharedPtr<FEngineInstallationInfo> > EngineInstallationInfos;
	bool Result;
};

static void Browse(FSelectBuildInfo& SelectBuildInfo)
{
	// Get the currently bound engine directory for the project
	const FString *RootDir = SelectBuildInfo.Installations.Find(SelectBuildInfo.Identifier);
	FString EngineRootDir = (RootDir != NULL)? *RootDir : FString();

	// Browse for a new directory
	FString NewEngineRootDir;
	if (!FDesktopPlatformModule::Get()->OpenDirectoryDialog(nullptr, TEXT("Select the Unreal Engine installation to use for this project"), EngineRootDir, NewEngineRootDir))
	{
		SelectBuildInfo.Result = false;
		return;
	}

	// Check it's a valid directory
	if (!FLinuxPlatformInstallation::NormalizeEngineRootDir(NewEngineRootDir))
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("The selected directory is not a valid engine installation."), TEXT("Error"));
		SelectBuildInfo.Result = false;
		return;
	}

	// Check that it's a registered engine directory
	FString NewIdentifier;
	if (!FDesktopPlatformModule::Get()->GetEngineIdentifierFromRootDir(NewEngineRootDir, NewIdentifier))
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Couldn't register engine installation."), TEXT("Error"));
		SelectBuildInfo.Result = false;
		return;
	}

	// Update the identifier and return
	SelectBuildInfo.Identifier = NewIdentifier;
	SelectBuildInfo.Result = true;
}

class SSelectBuildDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSelectBuildDialog)
		: _SelectBuildInfo(nullptr)
		, _ParentWindow(nullptr)
		, _StyleSet(nullptr)
	{}
	SLATE_ARGUMENT(FSelectBuildInfo*, SelectBuildInfo)
	SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)
	SLATE_ARGUMENT(FSlateFileDialogsStyle*, StyleSet)
	SLATE_END_ARGS()

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct(const FArguments& InArgs)
	{
		SelectBuildInfo = InArgs._SelectBuildInfo;
		ParentWindow = InArgs._ParentWindow;
		StyleSet = InArgs._StyleSet;

		// The code should already be greater than 0 if it got this far.
		verify(SelectBuildInfo->EngineInstallationInfos.Num() > 0);
		SelectBuildInfo->SelectedEngineInstallationInfo = SelectBuildInfo->EngineInstallationInfos[0];
		SelectBuildInfo->Identifier = SelectBuildInfo->SelectedEngineInstallationInfo->Identifier;

		this->ChildSlot
		[
			SNew(SBorder)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(FMargin(10.0f))
				.BorderImage(StyleSet->GetBrush("SlateFileDialogs.GroupBorder"))
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.Padding( 2.0f )
					[	
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
							.FillWidth(1)
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Center)
							.Padding(2.0f)
							[	
								SNew(SComboBox<TSharedPtr<FEngineInstallationInfo> >) // file list combo
									.OptionsSource(&SelectBuildInfo->EngineInstallationInfos)
									.OnGenerateWidget(this, &SSelectBuildDialog::OnGenerateWidget)
									.OnSelectionChanged(this, &SSelectBuildDialog::OnSelectionChanged)
									.InitiallySelectedItem(SelectBuildInfo->SelectedEngineInstallationInfo)
									[
										SNew(STextBlock)
											.Text(this, &SSelectBuildDialog::GetSelectedEngineInstallDescription)
									]
							]

						+SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							.Padding(2.0f)
							[	
								SNew(SButton)
									.HAlign(HAlign_Center)
									.Text(LOCTEXT("BrowseButton", "..."))
									.OnClicked(this, &SSelectBuildDialog::OnBrowseClicked)
							]
					]	

				+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(2.0f)
					[	
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SBox)
									.MinDesiredWidth(60.0f)
									[
										SNew(SButton)
											.HAlign(HAlign_Center)
											.Text(LOCTEXT("OkButton", "OK"))
											.OnClicked(this, &SSelectBuildDialog::OnOkClicked)
									]
							]

						+SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SBox)
									.MinDesiredWidth(60.0f)
									[
										SNew(SButton)
											.HAlign(HAlign_Center)
											.Text(LOCTEXT("CancelButton", "Cancel"))
											.OnClicked(this, &SSelectBuildDialog::OnCancelClicked)
									]
							]
					]
			]
		];
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:
	TSharedRef<SWidget> OnGenerateWidget(TSharedPtr<FEngineInstallationInfo> Item)
	{
		return SNew(STextBlock)
					.Text(FText::FromString(Item->Description));
	}

	FText GetSelectedEngineInstallDescription() const
	{
		return FText::FromString(SelectBuildInfo->SelectedEngineInstallationInfo->Description);
	}

	void OnSelectionChanged(TSharedPtr<FEngineInstallationInfo> Item, ESelectInfo::Type SelectInfo)
	{
		if (Item.IsValid())
		{
			SelectBuildInfo->Identifier = Item->Identifier;
			SelectBuildInfo->SelectedEngineInstallationInfo = Item;
		}
	}

	FReply OnBrowseClicked()
	{
		Browse(*SelectBuildInfo);
		if (SelectBuildInfo->Result)
		{
			ParentWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancelClicked()
	{
		SelectBuildInfo->Result = false;
		ParentWindow.Pin()->RequestDestroyWindow();
		return FReply::Handled();
	}

	FReply OnOkClicked()
	{
		SelectBuildInfo->Result = true;
		ParentWindow.Pin()->RequestDestroyWindow();
		return FReply::Handled();
	}

	FSelectBuildInfo* SelectBuildInfo;
	TWeakPtr<SWindow> ParentWindow;
	FSlateFileDialogsStyle* StyleSet;
};

class SErrorDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SErrorDialog)
		: _Message(FString())
		, _LogText(FString())
		, _ParentWindow(nullptr)
		, _StyleSet(nullptr)
	{}
	SLATE_ARGUMENT(FString, Message)
	SLATE_ARGUMENT(FString, LogText)
	SLATE_ARGUMENT(TWeakPtr<SWindow>, ParentWindow)
	SLATE_ARGUMENT(FSlateFileDialogsStyle*, StyleSet)
	SLATE_END_ARGS()

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct(const FArguments& InArgs)
	{
		Message = InArgs._Message;
		LogText = InArgs._LogText;
		ParentWindow = InArgs._ParentWindow;
		StyleSet = InArgs._StyleSet;
		
		this->ChildSlot
		[
			SNew(SBorder)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(FMargin(10.0f))
				.BorderImage(StyleSet->GetBrush("SlateFileDialogs.GroupBorder"))
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(2.0f)
					[	
						SNew(STextBlock)
						.Text(FText::FromString(InArgs._Message))
					]

				+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.Padding(2.0f)
					[
						SNew(SBorder)
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							.Padding(FMargin(10.0f))
							.BorderBackgroundColor(FLinearColor(0.40f, 0.40f, 0.40f, 1.0f))
						[
							SNew(SScrollBox)
								.Orientation(Orient_Horizontal)
								.ScrollBarAlwaysVisible(true)

							+SScrollBox::Slot()
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Fill)
								[
									SNew(SBox)
									.MinDesiredHeight(400.0f)
									[
										SNew(SEditableText)
											.IsReadOnly(true)
											.Text(FText::FromString(InArgs._LogText))
									]
								]
						]
					]

				+SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(2.0f)
					[
							
						SNew(SBox)
							.MinDesiredWidth(60.0f)
							[
								SNew(SButton)
									.HAlign(HAlign_Center)
									.Text(LOCTEXT("OkButton", "OK"))
									.OnClicked(this, &SErrorDialog::OnOkClicked)
							]
					]
			]
		];
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

private:
	FReply OnOkClicked()
	{
		ParentWindow.Pin()->RequestDestroyWindow();
		return FReply::Handled();
	}

	FString Message;
	FString LogText;
	TWeakPtr<SWindow> ParentWindow;
	FSlateFileDialogsStyle* StyleSet;
};

struct FSelectBuildDialog
{
	FString Identifier;

	FSelectBuildDialog(const FString& InIdentifier)
	{
		Identifier = InIdentifier;
		FDesktopPlatformModule::Get()->EnumerateEngineInstallations(SelectBuildInfo.Installations);
		SelectBuildInfo.Installations.GetKeys(SortedIdentifiers);
		SortedIdentifiers.Sort<FEngineLabelSortPredicate>(FEngineLabelSortPredicate());

		SelectBuildInfo.Identifier = Identifier;
		SelectBuildInfo.Result = false;

		for(int32 Idx =  0; Idx < SortedIdentifiers.Num(); Idx++)
		{
			const FString &SortedIdentifier = SortedIdentifiers[Idx];
			FString Description = GetInstallationDescription(SortedIdentifier, SelectBuildInfo.Installations[SortedIdentifier]);
			SelectBuildInfo.EngineInstallationInfos.Add(MakeShareable(new FEngineInstallationInfo{SortedIdentifier, Description}));
		}
	}

	bool DoModal()
	{
		// If there's more than one already, select them from a list
		if(SelectBuildInfo.Installations.Num() > 0)
		{
			return ShowDialog();
		}
		else if(FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, TEXT("No Unreal Engine installations found. Would you like to locate one manually?"), TEXT("Installation Not Found")) == EAppReturnType::Yes)
		{
			Browse(SelectBuildInfo);
			Identifier = SelectBuildInfo.Identifier;
			return SelectBuildInfo.Result;
		}
		return false;
	}

private:
	bool ShowDialog()
	{
		InitSlate();

		FSlateFileDialogsStyle StyleSet;
		StyleSet.Initialize();
	
		TSharedRef<SWindow> ModalWindow = SNew(SWindow)
			.SupportsMinimize(false)
			.SupportsMaximize(false)
			.Title(LOCTEXT("SelectBuild", "Select Unreal Engine Version"))
			.CreateTitleBar(true)
			.MinHeight(75.0f)
			.MinWidth(500.0f)
			.ActivationPolicy(EWindowActivationPolicy::Always)
			.ClientSize(FVector2D(500, 75));
		
		TSharedPtr<class SSelectBuildDialog> DialogWidget = SNew(SSelectBuildDialog)
			.SelectBuildInfo(&SelectBuildInfo)
			.ParentWindow(ModalWindow)
			.StyleSet(&StyleSet);
	
		ModalWindow->SetContent(DialogWidget.ToSharedRef());

		FSlateApplication::Get().AddModalWindow(ModalWindow, nullptr);

		CleanupSlate();

		Identifier = SelectBuildInfo.Identifier;
		return SelectBuildInfo.Result;
	}

	TArray<FString> SortedIdentifiers;
	FSelectBuildInfo SelectBuildInfo;
};

struct FErrorDialog
{
	FString Message, LogText;

	FErrorDialog(const FString& InMessage, const FString& InLogText) : Message(InMessage), LogText(InLogText)
	{

	}

	~FErrorDialog()
	{

	}

	bool DoModal()
	{
		return ShowDialog();
	}

private:
	bool ShowDialog()
	{
		InitSlate();

		FSlateFileDialogsStyle StyleSet;
		StyleSet.Initialize();

		TSharedRef<SWindow> ModalWindow = SNew(SWindow)
			.SupportsMinimize(false)
			.SupportsMaximize(false)
			.Title(LOCTEXT("Error", "Error"))
			.CreateTitleBar(true)
			.MinHeight(400.0f)
			.MinWidth(600.0f)
			.ActivationPolicy(EWindowActivationPolicy::Always)
			.ClientSize(FVector2D(800, 500));
	
		TSharedPtr<class SErrorDialog> DialogWidget = SNew(SErrorDialog)
			.Message(Message)
			.LogText(LogText)
			.ParentWindow(ModalWindow)
			.StyleSet(&StyleSet);
	
		ModalWindow->SetContent(DialogWidget.ToSharedRef());
		
		FSlateApplication::Get().AddModalWindow(ModalWindow, nullptr);

		CleanupSlate();

		return true;
	}
};

bool FLinuxPlatformInstallation::LaunchEditor(const FString& RootDirName, const FString &ExplicitFileName, const FString &Arguments)
{
	FString FileName;
	if (ExplicitFileName.IsEmpty())
	{
		FileName = RootDirName / TEXT("Engine/Binaries/Linux/UnrealEditor");
	}
	else
	{
		FileName = ExplicitFileName;
	}

	FProcHandle ProcessHandle = FPlatformProcess::CreateProc(*FileName, *Arguments, true, false, false, nullptr, 0, nullptr, nullptr);

	if (ProcessHandle.IsValid())
	{
		FPlatformProcess::CloseProc(ProcessHandle);
		return true;
	}

	return false;
}

bool FLinuxPlatformInstallation::SelectEngineInstallation(FString &Identifier)
{
	FSelectBuildDialog Dialog(Identifier);
	if (!Dialog.DoModal())
	{
		return false;
	}

	Identifier = Dialog.Identifier;
	return true;
}

void FLinuxPlatformInstallation::ErrorDialog(const FString &Message, const FString &LogText)
{
	FErrorDialog Dialog(Message, LogText);
	Dialog.DoModal();
}

#undef LOCTEXT_NAMESPACE
