// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "../GenericPlatform/GenericPlatformInstallation.h"

struct FLinuxPlatformInstallation : FGenericPlatformInstallation
{
	// Launches the editor application
	static bool LaunchEditor(const FString &RootDirName, const FString& ExplicitFileName, const FString &Arguments);

	// Select an engine installation
	static bool SelectEngineInstallation(FString &Identifier);

	// Shows an error dialog with log output
	static void ErrorDialog(const FString &Message, const FString &LogText);
};

