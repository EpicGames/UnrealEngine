// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UnixCommonStartup.h"

extern int32 UnrealVersionSelectorMain( const TCHAR* CommandLine );

TArray<FString> GArguments;

#pragma message("Remember to update the .desktop file's version, if making any significant changes to UVS.")

int main(int ArgC, char* ArgV[])
{
	for (int Idx = 1; Idx < ArgC; Idx++)
	{
		GArguments.Add(ArgV[Idx]);
	}
	
	return CommonUnixMain(1, ArgV, &UnrealVersionSelectorMain);
}
