// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if PLATFORM_WINDOWS
	#include "Windows/WindowsPlatformInstallation.h"
	typedef FWindowsPlatformInstallation FPlatformInstallation;
#elif PLATFORM_LINUX
	#include "Linux/LinuxPlatformInstallation.h"
	typedef FLinuxPlatformInstallation FPlatformInstallation;
#else
	#include "GenericPlatform/GenericPlatformInstallation.h"
	typedef FGenericPlatformInstallation FPlatformInstallation;
#endif
