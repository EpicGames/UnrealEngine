// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealVersionSelector.h"
#include "RequiredProgramMainCPPInclude.h"
#include "DesktopPlatformModule.h"
#include "PlatformInstallation.h"
#include "Misc/StringOutputDevice.h"
#include "Serialization/JsonSerializer.h"

IMPLEMENT_APPLICATION(UnrealVersionSelector, "UnrealVersionSelector")

bool GenerateProjectFiles(const FString& ProjectFileName);
bool UpdateFileAssociations();

bool RegisterCurrentEngineDirectory(bool bPromptForFileAssociations)
{
	// Get the current engine directory.
	FString EngineRootDir = FPlatformProcess::BaseDir();
	if(!FPlatformInstallation::NormalizeEngineRootDir(EngineRootDir))
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("The current folder does not contain an engine installation."), TEXT("Error"));
		return false;
	}

	// Get any existing tag name or register a new one
	FString Identifier;
	if (!FDesktopPlatformModule::Get()->GetEngineIdentifierFromRootDir(EngineRootDir, Identifier))
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Couldn't add engine installation."), TEXT("Error"));
		return false;
	}

	// If the launcher isn't installed, set up the file associations
	if(!FDesktopPlatformModule::Get()->VerifyFileAssociations())
	{
		// Prompt for whether to update the file associations
		if(!bPromptForFileAssociations || FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, TEXT("Register Unreal Engine file types?"), TEXT("File Types")) == EAppReturnType::Yes)
		{
#if PLATFORM_LINUX
			// Associations are set per user only so no need to elevate.
			return UpdateFileAssociations();
#else
			// Relaunch as administrator
			FString ExecutableFileName = FString(FPlatformProcess::BaseDir()) / FString(FPlatformProcess::ExecutableName(false));

			int32 ExitCode;
			if (!FPlatformProcess::ExecElevatedProcess(*ExecutableFileName, TEXT("/fileassociations"), &ExitCode) || ExitCode != 0)
			{
				return false;
			}
#endif
		}
	}

	return true;
}

bool RegisterCurrentEngineDirectoryWithPrompt()
{
	// Ask whether the user wants to register the directory
	if(FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, TEXT("Register this directory as an Unreal Engine installation?"), TEXT("Question")) != EAppReturnType::Yes)
	{
		return false;
	}

	// Register the engine directory. We've already prompted for registering the directory, so 
	if(!RegisterCurrentEngineDirectory(false))
	{
		return false;
	}

	// Notify the user that everything is awesome.
	FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Registration successful."), TEXT("Success"));
	return true;
}

bool UpdateFileAssociations()
{
	// Update everything
	if (!FDesktopPlatformModule::Get()->UpdateFileAssociations())
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Couldn't update file associations."), TEXT("Error"));
		return false;
	}
	return true;
}

bool SwitchVersion(const FString& ProjectFileName)
{
	// Get the current identifier
	FString Identifier;
	FDesktopPlatformModule::Get()->GetEngineIdentifierForProject(ProjectFileName, Identifier);

	// Select the new association
	if(!FPlatformInstallation::SelectEngineInstallation(Identifier))
	{
		return false;
	}

	// Update the project file
	if (!FDesktopPlatformModule::Get()->SetEngineIdentifierForProject(ProjectFileName, Identifier))
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Couldn't set association for project. Check if the uproject file contains a valid JSON and if the file is writeable."), TEXT("Error"));
		return false;
	}

	// If it's a content-only project, we're done
	FProjectStatus ProjectStatus;
	if(IProjectManager::Get().QueryStatusForProject(ProjectFileName, ProjectStatus) && !ProjectStatus.bCodeBasedProject)
	{
		return true;
	}

	// Generate project files
	return GenerateProjectFiles(ProjectFileName);
}

bool SwitchVersionSilent(const FString& ProjectFileName, const FString& IdentifierOrDirectory)
{
	// Convert the identifier or directory into an identifier
	FString Identifier = IdentifierOrDirectory;
	if (Identifier.Contains("\\") || Identifier.Contains("/"))
	{
		if (!FDesktopPlatformModule::Get()->GetEngineIdentifierFromRootDir(IdentifierOrDirectory, Identifier))
		{
			return false;
		}
	}

	// Update the project file
	if (!FDesktopPlatformModule::Get()->SetEngineIdentifierForProject(ProjectFileName, Identifier))
	{
		return false;
	}

	// If it's a content-only project, we're done
	FProjectStatus ProjectStatus;
	if(IProjectManager::Get().QueryStatusForProject(ProjectFileName, ProjectStatus) && !ProjectStatus.bCodeBasedProject)
	{
		return true;
	}

	// Generate project files
	return GenerateProjectFiles(ProjectFileName);
}

bool GetEngineRootDirForProject(const FString& ProjectFileName, FString& OutRootDir)
{
	FString Identifier;
	return FDesktopPlatformModule::Get()->GetEngineIdentifierForProject(ProjectFileName, Identifier) && FDesktopPlatformModule::Get()->GetEngineRootDirFromIdentifier(Identifier, OutRootDir);
}

bool GetValidatedEngineRootDir(const FString& ProjectFileName, FString& OutRootDir)
{
	// Get the engine directory for this project
	if (!GetEngineRootDirForProject(ProjectFileName, OutRootDir))
	{
		// Try to set an association
		if(!SwitchVersion(ProjectFileName))
		{
			return false;
		}

		// See if it's valid now
		if (!GetEngineRootDirForProject(ProjectFileName, OutRootDir))
		{
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Error retrieving project root directory"), TEXT("Error"));
			return false;
		}
	}
	return true;
}

bool LaunchEditor()
{
	FString Identifier;

	// Select which editor to launch
	if(!FPlatformInstallation::SelectEngineInstallation(Identifier))
	{
		return false;
	}

	FString RootDir;
	FDesktopPlatformModule::Get()->GetEngineRootDirFromIdentifier(Identifier, RootDir);

	// Launch the editor
	if (!FPlatformInstallation::LaunchEditor(RootDir, FString(), FString()))
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Failed to launch editor"), TEXT("Error"));
		return false;
	}

	return true;
}

bool ReadLaunchPathFromTargetFile(const FString& TargetFileName, FString& OutLaunchPath)
{
	// Read the file to a string
	FString FileContents;
	if (!FFileHelper::LoadFileToString(FileContents, *TargetFileName))
	{
		return false;
	}

	// Deserialize a JSON object from the string
	TSharedPtr<FJsonObject> Object;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContents);
	if (!FJsonSerializer::Deserialize(Reader, Object) || !Object.IsValid())
	{
		return false;
	}

	// Check it's an editor target
	FString TargetType;
	if(!Object->TryGetStringField(TEXT("TargetType"), TargetType) || TargetType != TEXT("Editor"))
	{
		return false;
	}

	// Check it's development configuration
	FString Configuration;
	if (!Object->TryGetStringField(TEXT("Configuration"), Configuration) || Configuration != TEXT("Development"))
	{
		return false;
	}

	// Get the launch path
	OutLaunchPath.Empty();
	Object->TryGetStringField(TEXT("Launch"), OutLaunchPath);
	return true;
}

bool TryGetEditorFileName(const FString& EngineDir, const FString& ProjectFileName, FString& OutEditorFileName)
{
	IFileManager& FileManager = IFileManager::Get();

	FString ProjectDir = FPaths::GetPath(ProjectFileName);
	FString BinariesDir = ProjectDir / TEXT("Binaries") / FPlatformProcess::GetBinariesSubdirectory();
	if (FileManager.DirectoryExists(*BinariesDir))
	{
		class FTargetFileVisitor : public IPlatformFile::FDirectoryStatVisitor
		{
		public:
			TArray<TPair<FString, FDateTime>> Files;

			virtual bool Visit(const TCHAR* FilenameOrDirectory, const FFileStatData& StatData)
			{
				static const TCHAR Extension[] = TEXT(".target");
				static const int ExtensionLen = UE_ARRAY_COUNT(Extension) - 1;

				int Length = FCString::Strlen(FilenameOrDirectory);
				if(Length >= ExtensionLen && FCString::Stricmp(FilenameOrDirectory + Length - ExtensionLen, Extension) == 0)
				{
					Files.Add(TPair<FString, FDateTime>(FilenameOrDirectory, StatData.ModificationTime));
				}

				return true;
			}
		};

		FTargetFileVisitor Visitor;
		FileManager.IterateDirectoryStat(*BinariesDir, Visitor);

		Visitor.Files.Sort([](const TPair<FString, FDateTime>& A, const TPair<FString, FDateTime>& B){ return A.Value > B.Value; });

		for(const TPair<FString, FDateTime>& Pair : Visitor.Files)
		{
			FString LaunchPath;
			if(ReadLaunchPathFromTargetFile(Pair.Key, LaunchPath))
			{
				OutEditorFileName = MoveTemp(LaunchPath);
				OutEditorFileName.ReplaceInline(TEXT("$(EngineDir)"), *EngineDir);
				OutEditorFileName.ReplaceInline(TEXT("$(ProjectDir)"), *ProjectDir);
				return true;
			}
		}
	}
	return false;
}

bool LaunchEditor(const FString& ProjectFileName, const FString& Arguments)
{
	// Get the engine root directory
	FString RootDir;
	if (!GetValidatedEngineRootDir(ProjectFileName, RootDir))
	{
		return false;
	}

	// Figure out the path to the editor executable. This may be empty for older .target files; the platform layer should use the default path if necessary.
	FString EditorFileName;
	TryGetEditorFileName(RootDir / TEXT("Engine"), ProjectFileName, EditorFileName);

	// Launch the editor
	if (!FPlatformInstallation::LaunchEditor(RootDir, EditorFileName, FString::Printf(TEXT("\"%s\" %s"), *ProjectFileName, *Arguments)))
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Failed to launch editor"), TEXT("Error"));
		return false;
	}

	return true;
}

bool GenerateProjectFiles(const FString& ProjectFileName)
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	// Check it's a code project
	FString SourceDir = FPaths::GetPath(ProjectFileName) / TEXT("Source");
	if(!IPlatformFile::GetPlatformPhysical().DirectoryExists(*SourceDir))
	{
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("This project does not have any source code. You need to add C++ source files to the project from the Editor before you can generate project files."), TEXT("Error"));
		return false;
	}

	// Get the engine root directory
	FString RootDir;
	if (!GetValidatedEngineRootDir(ProjectFileName, RootDir))
	{
		return false;
	}

	// Start capturing the log output
	FStringOutputDevice LogCapture;
	LogCapture.SetAutoEmitLineTerminator(true);
	GLog->AddOutputDevice(&LogCapture);

	// Generate project files
	FFeedbackContext* Warn = DesktopPlatform->GetNativeFeedbackContext();
	bool bResult = DesktopPlatform->GenerateProjectFiles(RootDir, ProjectFileName, Warn, FString::Printf(TEXT("%s/Saved/Logs/%s-%s.log"), *FPaths::GetPath(ProjectFileName), FPlatformProcess::ExecutableName(), *FDateTime::Now().ToString()));
	GLog->RemoveOutputDevice(&LogCapture);

	// Display an error dialog if we failed
	if(!bResult)
	{
		FPlatformInstallation::ErrorDialog(TEXT("Failed to generate project files."), LogCapture);
		return false;
	}

	return true;
}

int Main(const TArray<FString>& Arguments)
{
	bool bRes = false;
	if (Arguments.Num() == 0)
	{
		// Add the current directory to the list of installations
		bRes = RegisterCurrentEngineDirectoryWithPrompt();
	}
	else if (Arguments.Num() == 1 && Arguments[0] == TEXT("-register"))
	{
		// Add the current directory to the list of installations
		bRes = RegisterCurrentEngineDirectory(true);
	}
	else if (Arguments.Num() == 2 && Arguments[0] == TEXT("-register") && Arguments[1] == TEXT("-unattended"))
	{
		// Add the current directory to the list of installations
		bRes = RegisterCurrentEngineDirectory(false);
	}
	else if (Arguments.Num() == 1 && Arguments[0] == TEXT("-fileassociations"))
	{
		// Update all the settings.
		bRes = UpdateFileAssociations();
	}
	else if (Arguments.Num() == 2 && Arguments[0] == TEXT("-switchversion"))
	{
		// Associate with an engine label
		bRes = SwitchVersion(Arguments[1]);
	}
	else if (Arguments.Num() == 3 && Arguments[0] == TEXT("-switchversionsilent"))
	{
		// Associate with a specific engine label
		bRes = SwitchVersionSilent(Arguments[1], Arguments[2]);
	}
	else if (Arguments.Num() == 2 && Arguments[0] == TEXT("-editor"))
	{
		// Open a project with the editor
		bRes = LaunchEditor(Arguments[1], TEXT(""));
	}
	else if (Arguments[0] == TEXT("-projectlist"))
	{
		// Open the editor
		bRes = LaunchEditor();
	}
	else if (Arguments.Num() == 2 && Arguments[0] == TEXT("-game"))
	{
		// Play a game using the editor executable
		bRes = LaunchEditor(Arguments[1], TEXT("-game"));
	}
	else if (Arguments.Num() == 2 && Arguments[0] == TEXT("-projectfiles"))
	{
		// Generate Visual Studio project files
		bRes = GenerateProjectFiles(Arguments[1]);
	}
	else
	{
		// Invalid command line
		FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Invalid command line"), NULL);
	}
	return bRes ? 0 : 1;
}

#if PLATFORM_WINDOWS

	#include "Windows/AllowWindowsPlatformTypes.h"
	#include <Shellapi.h>

	int WINAPI WinMain(_In_ HINSTANCE /*hInstance*/, _In_opt_ HINSTANCE /*hPrevInstance*/, _In_ LPSTR /*lpCmdLine*/, _In_ int /*nShowCmd*/)
	{
		int ArgC;
		LPWSTR* ArgV = ::CommandLineToArgvW(GetCommandLineW(), &ArgC);

		FCommandLine::Set(TEXT(""));

		TArray<FString> Arguments;
		for (int Idx = 1; Idx < ArgC; Idx++)
		{
			FString Argument = ArgV[Idx];
			if(Argument.Len() > 0 && Argument[0] == '/')
			{
				Argument[0] = '-';
			}
			Arguments.Add(Argument);
		}

		return Main(Arguments);
	}

	#include "Windows/HideWindowsPlatformTypes.h"

#elif PLATFORM_LINUX

	extern TArray<FString> GArguments;

	int32 UnrealVersionSelectorMain( const TCHAR* CommandLine )
	{
		FCommandLine::Set(CommandLine);

		GEngineLoop.PreInit(CommandLine);

		ProcessNewlyLoadedUObjects();

		FModuleManager::Get().StartProcessingNewlyLoadedObjects();

		int32 Result = Main(GArguments);

		FEngineLoop::AppPreExit();
		FModuleManager::Get().UnloadModulesAtShutdown();
		FEngineLoop::AppExit();

		return Result;
	}

#else

	int main(int ArgC, const char* ArgV[])
	{
		FCommandLine::Set(TEXT(""));
		
		TArray<FString> Arguments;
		for (int Idx = 1; Idx < ArgC; Idx++)
		{
			Arguments.Add(ArgV[Idx]);
		}
		
		return Main(Arguments);
	}

#endif
