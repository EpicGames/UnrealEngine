// Copyright Epic Games, Inc. All Rights Reserved.

#include "WindowsPlatformInstallation.h"
#include "Misc/EngineVersion.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include "DesktopPlatformModule.h"
#include "Misc/Paths.h"
#include "Resources/Resource.h"
#include <CommCtrl.h>
#include <Shlwapi.h>
#include <Shellapi.h>

struct FEngineLabelSortPredicate
{
	bool operator()(const FString &A, const FString &B) const
	{
		return FDesktopPlatformModule::Get()->IsPreferredEngineIdentifier(A, B);
	}
};

FString GetInstallationDescription(const FString &Id, const FString &RootDir)
{
	// Official release versions just have a version number
	if (Id.Len() > 0 && FChar::IsDigit(Id[0]))
	{
		return Id;
	}

	// Otherwise get the path
	FString PlatformRootDir = RootDir;
	FPaths::MakePlatformFilename(PlatformRootDir);

	// Perforce build
	if (FDesktopPlatformModule::Get()->IsSourceDistribution(RootDir))
	{
		return FString::Printf(TEXT("Source build at %s"), *PlatformRootDir);
	}
	else
	{
		return FString::Printf(TEXT("Binary build at %s"), *PlatformRootDir);
	}
}

struct FSelectBuildDialog
{
	FString Identifier;

	FSelectBuildDialog(const FString &InIdentifier)
	{
		Identifier = InIdentifier;
		FDesktopPlatformModule::Get()->EnumerateEngineInstallations(Installations);
		Installations.GetKeys(SortedIdentifiers);
		SortedIdentifiers.Sort<FEngineLabelSortPredicate>(FEngineLabelSortPredicate());
	}

	bool DoModal(HWND hWndParent)
	{
		// If there's more than one already, select them from a list
		if(Installations.Num() > 0)
		{
			return DialogBoxParam(GetModuleHandle(NULL), MAKEINTRESOURCE(IDD_SELECTBUILD), hWndParent, &DialogProc, (LPARAM)this) != FALSE;
		}
		else if(FPlatformMisc::MessageBoxExt(EAppMsgType::YesNo, TEXT("No Unreal Engine installations found. Would you like to locate one manually?"), TEXT("Installation Not Found")) == EAppReturnType::Yes && Browse(hWndParent))
		{
			return true;
		}
		return false;
	}

private:
	TArray<FString> SortedIdentifiers;
	TMap<FString, FString> Installations;

	static INT_PTR CALLBACK DialogProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		FSelectBuildDialog *Dialog = (FSelectBuildDialog*)GetWindowLongPtr(hWnd, GWLP_USERDATA);

		switch (Msg)
		{
		case WM_INITDIALOG:
			Dialog = (FSelectBuildDialog*)lParam;
			SetWindowLongPtr(hWnd, GWLP_USERDATA, (LONG_PTR)Dialog);
			Dialog->UpdateInstallations(hWnd);
			return false;
		case WM_COMMAND:
			switch (LOWORD(wParam))
			{
			case IDC_BROWSE:
				if (Dialog->Browse(hWnd))
				{
					EndDialog(hWnd, 1);
				}
				return false;
			case IDOK:
				Dialog->StoreSelection(hWnd);
				EndDialog(hWnd, 1);
				return false;
			case IDCANCEL:
				EndDialog(hWnd, 0);
				return false;
			}

		}
		return FALSE;
	}

	void StoreSelection(HWND hWnd)
	{
		int32 Idx = SendDlgItemMessage(hWnd, IDC_BUILDLIST, CB_GETCURSEL, 0, 0);
		Identifier = SortedIdentifiers.IsValidIndex(Idx) ? SortedIdentifiers[Idx] : TEXT("");
	}

	void UpdateInstallations(HWND hWnd)
	{
		SendDlgItemMessage(hWnd, IDC_BUILDLIST, CB_RESETCONTENT, 0, 0);

		for(int32 Idx =  0; Idx < SortedIdentifiers.Num(); Idx++)
		{
			const FString &SortedIdentifier = SortedIdentifiers[Idx];
			FString Description = GetInstallationDescription(SortedIdentifier, Installations[SortedIdentifier]);
			SendDlgItemMessage(hWnd, IDC_BUILDLIST, CB_ADDSTRING, 0, (LPARAM)*Description);
		}

		int32 NewIdx = FMath::Max(SortedIdentifiers.IndexOfByKey(Identifier), 0);
		SendDlgItemMessage(hWnd, IDC_BUILDLIST, CB_SETCURSEL, NewIdx, 0);
	}

	bool Browse(HWND hWnd)
	{
		// Get the currently bound engine directory for the project
		const FString *RootDir = Installations.Find(Identifier);
		FString EngineRootDir = (RootDir != NULL)? *RootDir : FString();

		// Browse for a new directory
		FString NewEngineRootDir;
		if (!FDesktopPlatformModule::Get()->OpenDirectoryDialog(hWnd, TEXT("Select the Unreal Engine installation to use for this project"), EngineRootDir, NewEngineRootDir))
		{
			return false;
		}

		// Check it's a valid directory
		if (!FWindowsPlatformInstallation::NormalizeEngineRootDir(NewEngineRootDir))
		{
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("The selected directory is not a valid engine installation."), TEXT("Error"));
			return false;
		}

		// Check that it's a registered engine directory
		FString NewIdentifier;
		if (!FDesktopPlatformModule::Get()->GetEngineIdentifierFromRootDir(NewEngineRootDir, NewIdentifier))
		{
			FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, TEXT("Couldn't register engine installation."), TEXT("Error"));
			return false;
		}

		// Update the identifier and return
		Identifier = NewIdentifier;
		return true;
	}
};

struct FErrorDialog
{
	HFONT hFont;
	FString Message, LogText;

	FErrorDialog(const FString& InMessage, const FString& InLogText) : Message(InMessage), LogText(InLogText)
	{
		int FontHeight = -MulDiv(8, GetDeviceCaps(GetDC(NULL), LOGPIXELSY), 72);
		hFont = CreateFontW(FontHeight, 0, 0, 0, FW_NORMAL, FALSE, FALSE, FALSE, ANSI_CHARSET, OUT_DEFAULT_PRECIS, CLIP_DEFAULT_PRECIS, ANTIALIASED_QUALITY, FIXED_PITCH | FF_MODERN, TEXT("Courier New"));
	}

	~FErrorDialog()
	{
		DeleteObject(hFont);
	}

	bool DoModal(HWND hWndParent)
	{
		return DialogBoxParam(GetModuleHandle(NULL), MAKEINTRESOURCE(IDD_ERRORDIALOG), hWndParent, &DialogProc, (LPARAM)this) != FALSE;
	}

private:
	static INT_PTR CALLBACK DialogProc(HWND hWnd, UINT Msg, WPARAM wParam, LPARAM lParam)
	{
		FErrorDialog *Dialog = (FErrorDialog*)GetWindowLongPtr(hWnd, GWLP_USERDATA);

		switch (Msg)
		{
		case WM_INITDIALOG:
			Dialog = (FErrorDialog*)lParam;
			SetWindowLongPtr(hWnd, GWLP_USERDATA, (LONG_PTR)Dialog);

			SetDlgItemText(hWnd, IDC_ERRORMESSAGE, *Dialog->Message);
			SetDlgItemText(hWnd, IDC_ERRORLOGTEXT, *Dialog->LogText);

			SendDlgItemMessage(hWnd, IDC_ERRORLOGTEXT, WM_SETFONT, (WPARAM)Dialog->hFont, 0);
			SendDlgItemMessage(hWnd, IDC_ERRORLOGTEXT, EM_LINESCROLL, 0, 32000);
			return FALSE;
		case WM_COMMAND:
			switch (LOWORD(wParam))
			{
			case IDOK:
				EndDialog(hWnd, 1);
				return FALSE;
			}
			return FALSE;
		case WM_CLOSE:
			EndDialog(hWnd, 1);
			return FALSE;
		}
		return FALSE;
	}
};

bool FWindowsPlatformInstallation::LaunchEditor(const FString &RootDirName, const FString& ExplicitFileName, const FString &Arguments)
{
	FString FileName;
	if (ExplicitFileName.IsEmpty())
	{
		FileName = RootDirName / TEXT("Engine/Binaries/Win64/UnrealEditor.exe");
	}
	else
	{
		FileName = ExplicitFileName;
	}

	FString CommandLine = FString::Printf(TEXT("\"%s\" %s"), *FileName, *Arguments);

	STARTUPINFO StartupInfo;
	ZeroMemory(&StartupInfo, sizeof(StartupInfo));
	StartupInfo.cb = sizeof(StartupInfo);

	PROCESS_INFORMATION ProcessInfo;
	if(!CreateProcess(NULL, CommandLine.GetCharArray().GetData(), NULL, NULL, FALSE, 0, NULL, NULL, &StartupInfo, &ProcessInfo))
	{
		return false;
	}

	CloseHandle(ProcessInfo.hThread);
	CloseHandle(ProcessInfo.hProcess);
	return true;
}

bool FWindowsPlatformInstallation::SelectEngineInstallation(FString &Identifier)
{
	FSelectBuildDialog Dialog(Identifier);
	if(!Dialog.DoModal(NULL))
	{
		return false;
	}

	Identifier = Dialog.Identifier;
	return true;
}

void FWindowsPlatformInstallation::ErrorDialog(const FString &Message, const FString &LogText)
{
	FErrorDialog Dialog(Message, LogText);
	Dialog.DoModal(NULL);
}

#include "Windows/HideWindowsPlatformTypes.h"
