// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnrealVersionSelector : ModuleRules
{
	public UnrealVersionSelector(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			PrivateDependencyModuleNames.AddRange(new string[] { "Core", "Projects", "DesktopPlatform", "CoreUObject", "ApplicationCore", "Slate", "SlateCore", "StandaloneRenderer", "SlateFileDialogs" });
			PrivateDependencyModuleNames.AddRange(new string[] { "UnixCommonStartup" });
 		}
		else
		{
			PrivateDependencyModuleNames.AddRange(new string[] { "Core", "Projects", "DesktopPlatform", "ApplicationCore" });
		}

        PrivateDependencyModuleNames.Add("Json");
    }
}
