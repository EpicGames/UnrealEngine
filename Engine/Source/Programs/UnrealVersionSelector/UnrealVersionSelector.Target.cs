// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

public class UnrealVersionSelectorTarget : TargetRules
{
	public UnrealVersionSelectorTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LaunchModuleName = "UnrealVersionSelector";
		
		bBuildDeveloperTools = false;

		bool bUsingSlate = (Target.Platform == UnrealTargetPlatform.Linux);

		if (bUsingSlate && bBuildEditor)
		{
			ExtraModuleNames.Add("EditorStyle");
		}

		bCompileICU = false;
		// Editor-only data, however, is needed
		bBuildWithEditorOnlyData = bUsingSlate;

		// Currently this app is not linking against the engine, so we'll compile out references from Core to the rest of the engine
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = bUsingSlate;
	}
}
